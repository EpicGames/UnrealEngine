# Copyright Epic Games, Inc. All Rights Reserved.

get_directory_property(HAS_PARENT PARENT_DIRECTORY)
if(HAS_PARENT)
  message(FATAL_ERROR "This CMakeLists cannot be included in another CMake project!")
endif()

string(TOUPPER "${CMAKE_BUILD_TYPE}" CMAKE_BUILD_TYPE)

if(DEFINED CMAKE_CONFIGURATION_TYPES)
  message(FATAL_ERROR "Cannot use multi-configuration generator '${CMAKE_GENERATOR}' with the LLVM builds (recommend you use 'Ninja')!")
endif()

set(BUILD_TYPE_TO_USE "RELEASE")

if(NOT "${CMAKE_BUILD_TYPE}" STREQUAL "${BUILD_TYPE_TO_USE}")
  message(WARNING "CMAKE_BUILD_TYPE was '${CMAKE_BUILD_TYPE}'. Overriding to '${BUILD_TYPE_TO_USE}'!")

  set(CMAKE_BUILD_TYPE "${BUILD_TYPE_TO_USE}" CACHE STRING "" FORCE)
endif()

cmake_minimum_required(VERSION 3.19)
project(BuildLLVMs)

# The UnrealInstrumentation directory is one up from our current directory, so find that now.
set(UNREALINSTRUMENTATION_DIR "${CMAKE_CURRENT_SOURCE_DIR}")
string(FIND "${UNREALINSTRUMENTATION_DIR}" "/" LAST_SEP REVERSE)
string(SUBSTRING "${UNREALINSTRUMENTATION_DIR}" 0 "${LAST_SEP}" UNREALINSTRUMENTATION_DIR)

find_package(Git)
if(NOT Git_FOUND)
  message(FATAL_ERROR "The executable 'git' was not found on the PATH, please add it!")
endif()

find_package(Patch)
if(NOT Patch_FOUND)
  message(FATAL_ERROR "The executable 'patch' was not found on the PATH, please add it!")
endif()

find_program(P4_EXECUTABLE p4)
if("${P4_EXECUTABLE}" STREQUAL "P4_EXECUTABLE-NOTFOUND")
  message(FATAL_ERROR "The executable 'p4' was not found on the PATH, please add it!")
endif()

find_package(Python)
if(NOT Python_FOUND)
  message(FATAL_ERROR "The executable 'python' was not found on the PATH, please add it!")
endif()

include(ExternalProject)
include(FetchContent)

# This property just makes the external project's output folders in a much nicer
# directory layout on disk.
set_property(DIRECTORY PROPERTY EP_BASE "${CMAKE_CURRENT_BINARY_DIR}")

set(UE_SDKS_ROOT $ENV{UE_SDKS_ROOT})

if("${UE_SDKS_ROOT}" STREQUAL "")
  message(FATAL_ERROR "The environment variable 'UE_SDKS_ROOT' was not set!")
endif()

set(CAREFULLY_REDIST_DIR "${UE_SDKS_ROOT}/HostWin64")
set(CAREFULLY_REDIST_WIN64_DIR "${CAREFULLY_REDIST_DIR}/Win64")
file(TO_CMAKE_PATH "${CAREFULLY_REDIST_WIN64_DIR}" CAREFULLY_REDIST_WIN64_DIR)

set(BOOTSTRAP_DIR ${UE_SDKS_ROOT}/HostWin64/Win64/LLVM/20.1.8)
file(TO_CMAKE_PATH "${BOOTSTRAP_DIR}" BOOTSTRAP_DIR)

if(WIN32)
  # This is the folder that the external project builds rpmalloc into.
  set(RPMALLOC_SOURCE_DIR ${CMAKE_CURRENT_BINARY_DIR}/Src/rpmalloc)

  FetchContent_Declare(
    rpmalloc
    GIT_REPOSITORY https://github.com/mjansson/rpmalloc.git
    GIT_TAG main
    SOURCE_DIR ${RPMALLOC_SOURCE_DIR}
  )

  FetchContent_MakeAvailable(rpmalloc)
endif()

file(REAL_PATH "${CMAKE_CURRENT_SOURCE_DIR}/../../../Source/ThirdParty/zlib/1.3" ZLIB_DIR)

function(build)
  set(oneValueArgs NAME GIT_VERSION PATCH_PROJECT_TO_USE)
  set(multiValueArgs "")
  cmake_parse_arguments(BUILD "${options}" "${oneValueArgs}" "${multiValueArgs}" ${ARGN})

  set(NAME "${BUILD_NAME}")
  set(GIT_VERSION "${BUILD_GIT_VERSION}")
  set(PATCH_PROJECT_TO_USE "${BUILD_PATCH_PROJECT_TO_USE}")

  if("${NAME}" STREQUAL "")
    message(FATAL_ERROR "Non-optional argument 'NAME' was not provided!")
  endif()

  if("${GIT_VERSION}" STREQUAL "")
    message(FATAL_ERROR "Non-optional argument 'GIT_VERSION' was not provided!")
  endif()

  if("${PATCH_PROJECT_TO_USE}" STREQUAL "")
    set(PATCH_PROJECT_TO_USE "${NAME}")
  endif()

  set(PATCH_FILE "${CMAKE_CURRENT_SOURCE_DIR}/llvm-project-${PATCH_PROJECT_TO_USE}.patch")

  if(NOT EXISTS "${PATCH_FILE}")
    message(FATAL_ERROR "The patch file does not exist!")
  endif()

  set(EXTRA_COMPILER_FLAGS "")

  set(EXTRA_COMPILER_FLAGS
    ${EXTRA_COMPILER_FLAGS}
    -DCMAKE_C_COMPILER=${BOOTSTRAP_DIR}/bin/clang-cl.exe
    -DCMAKE_CXX_COMPILER=${BOOTSTRAP_DIR}/bin/clang-cl.exe
    -DCMAKE_LINKER=${BOOTSTRAP_DIR}/bin/lld-link.exe
    -DCMAKE_AR=${BOOTSTRAP_DIR}/bin/llvm-lib.exe
  )

  if(DO_LTO)
    set(EXTRA_COMPILER_FLAGS
      ${EXTRA_COMPILER_FLAGS}
      -DLLVM_ENABLE_LTO=Thin
    )
  endif()

  set(PATCH_HASH_FILE ${CMAKE_CURRENT_BINARY_DIR}/Build/llvm-${NAME}/patch.hash)
  set(ENABLE_CUSTOM_ALLOCATOR "-DLLVM_INTEGRATED_CRT_ALLOC=${RPMALLOC_SOURCE_DIR}")

  set(CLANG_VENDOR "Epic Games, Inc.")

  ExternalProject_Add(llvm-${NAME}
    GIT_REPOSITORY ${GIT_REPOSITORY}
    GIT_TAG ${GIT_VERSION}
    PATCH_COMMAND ${CMAKE_COMMAND}
      -DUNREALINSTRUMENTATION_DIR=${UNREALINSTRUMENTATION_DIR}
      -DPATCH_FILE=${PATCH_FILE}
      -DPATCH_HASH_FILE=${PATCH_HASH_FILE}
      -DGIT_EXECUTABLE=${GIT_EXECUTABLE}
      -DPatch_EXECUTABLE=${Patch_EXECUTABLE}
      -P ${CMAKE_CURRENT_SOURCE_DIR}/cmake/PatchIfClean.cmake

    SOURCE_SUBDIR "llvm"
    CMAKE_ARGS
      -DUNREALINSTRUMENTATION_DIR=${UNREALINSTRUMENTATION_DIR}
      -DCMAKE_BUILD_TYPE:STRING=${CMAKE_BUILD_TYPE}
      -DCMAKE_INSTALL_PREFIX=${CMAKE_CURRENT_BINARY_DIR}/Install/llvm-${NAME}
      -DLLVM_ENABLE_PROJECTS=clang$<SEMICOLON>lld
      -DLLVM_TARGETS_TO_BUILD=X86

      # We need to force disable assertions because the static analyzer has bugs
      # in Epic code that are only caught by assertions!
      -DLLVM_ENABLE_ASSERTIONS=OFF

      # enable generating debuginfo from RELEASE
      -DLLVM_ENABLE_PDB=OFF

      # Always statically link the CRT on Windows.
      -DLLVM_USE_CRT_${CMAKE_BUILD_TYPE}=MT
      -DCMAKE_MSVC_RUNTIME_LIBRARY=MultiThreaded

      # Force C++17
      -DCMAKE_CXX_STANDARD=17

      # Use a custom allocator to improve the performance on Windows.
      ${ENABLE_CUSTOM_ALLOCATOR}

      # Add any remaining extra flags too.
      ${EXTRA_COMPILER_FLAGS}

      # We want to register that our Clang is different from other Clang's.
      "-DCLANG_VENDOR=${CLANG_VENDOR}"

      # We need to build against zlib for debug info compression.
      -DLLVM_ENABLE_ZLIB=FORCE_ON
      -DZLIB_LIBRARY:PATH=${ZLIB_DIR}/lib/Win64/Release/zlibstatic.lib
      -DZLIB_INCLUDE_DIR:PATH=${ZLIB_DIR}/include
      -DHAVE_ZLIB=1
    STEP_TARGETS update configure install
    TEST_BEFORE_INSTALL false
  )

  # Add a step to sync CustomMemoryInstrumentation files before building.
  # This enables incremental rebuilds when only those files change.
  set(CUSTOM_INSTRUMENTATION_CPP "${CMAKE_CURRENT_SOURCE_DIR}/CustomMemoryInstrumentation.cpp")
  set(CUSTOM_INSTRUMENTATION_H "${CMAKE_CURRENT_SOURCE_DIR}/CustomMemoryInstrumentation.h")
  set(LLVM_INSTRUMENTATION_SRC_DIR "${CMAKE_CURRENT_BINARY_DIR}/Src/llvm-${NAME}/llvm/lib/Transforms/Instrumentation")
  set(LLVM_INSTRUMENTATION_INC_DIR "${CMAKE_CURRENT_BINARY_DIR}/Src/llvm-${NAME}/llvm/include/llvm/Transforms/Instrumentation")

  ExternalProject_Add_Step(llvm-${NAME} sync_custom_instrumentation
    COMMAND ${CMAKE_COMMAND} -E copy_if_different "${CUSTOM_INSTRUMENTATION_CPP}" "${LLVM_INSTRUMENTATION_SRC_DIR}"
    COMMAND ${CMAKE_COMMAND} -E copy_if_different "${CUSTOM_INSTRUMENTATION_H}" "${LLVM_INSTRUMENTATION_INC_DIR}"
    DEPENDERS build
    DEPENDEES patch
    DEPENDS "${CUSTOM_INSTRUMENTATION_CPP}" "${CUSTOM_INSTRUMENTATION_H}"
    ALWAYS TRUE
    COMMENT "Syncing CustomMemoryInstrumentation files..."
  )

  # We add an extra step that runs after installation to copy the clang
  # executables to instr-<name> to make it super clear that this is a custom
  # variant of clang with extra instr sauce in it!
  ExternalProject_Add_Step(llvm-${NAME} binaries_renaming
    COMMAND ${CMAKE_COMMAND} -E copy clang++${CMAKE_EXECUTABLE_SUFFIX} instr-clang++${CMAKE_EXECUTABLE_SUFFIX}
    COMMAND ${CMAKE_COMMAND} -E copy clang-cl${CMAKE_EXECUTABLE_SUFFIX} instr-clang-cl${CMAKE_EXECUTABLE_SUFFIX}
    COMMAND ${CMAKE_COMMAND} -E copy clang${CMAKE_EXECUTABLE_SUFFIX} instr-clang${CMAKE_EXECUTABLE_SUFFIX}
    DEPENDEES install
    WORKING_DIRECTORY "${CMAKE_CURRENT_BINARY_DIR}/Install/llvm-${NAME}/bin"
  )

  ExternalProject_Add_StepTargets(llvm-${NAME} binaries_renaming)

endfunction()

function(deploy)
  set(options)
  set(oneValueArgs NAME VERSION)
  set(multiValueArgs "")
  cmake_parse_arguments(DEPLOY "${options}" "${oneValueArgs}" "${multiValueArgs}" ${ARGN})

  set(NAME "${DEPLOY_NAME}")
  set(VERSION "${DEPLOY_VERSION}")

  set(LLVM_DIR ${CMAKE_CURRENT_BINARY_DIR}/Install/llvm-${NAME})
  set(LLVM_BUILD_DIR ${CMAKE_CURRENT_BINARY_DIR}/Build/llvm-${NAME})

  file(REAL_PATH "${CMAKE_CURRENT_SOURCE_DIR}/../../../Binaries/Win64/UnrealInstrumentation" P4_DIR)

  add_custom_target(deploy-${NAME}-main
    COMMAND ${P4_EXECUTABLE} edit
      "${P4_DIR}/lib/clang/..."
    COMMAND ${CMAKE_COMMAND} -E remove_directory
      "${P4_DIR}/lib/clang"
    COMMAND ${P4_EXECUTABLE} edit
      "${P4_DIR}/bin/instr-clang-cl${CMAKE_EXECUTABLE_SUFFIX}"
    COMMAND ${CMAKE_COMMAND} -E copy
      "${LLVM_DIR}/bin/instr-clang-cl${CMAKE_EXECUTABLE_SUFFIX}"
      "${P4_DIR}/bin/instr-clang-cl${CMAKE_EXECUTABLE_SUFFIX}"
    COMMAND ${P4_EXECUTABLE} edit
      "${P4_DIR}/bin/ld.lld${CMAKE_EXECUTABLE_SUFFIX}"
    COMMAND ${CMAKE_COMMAND} -E copy
      "${LLVM_DIR}/bin/ld.lld${CMAKE_EXECUTABLE_SUFFIX}"
      "${P4_DIR}/bin/ld.lld${CMAKE_EXECUTABLE_SUFFIX}"
    COMMAND ${P4_EXECUTABLE} edit
      "${P4_DIR}/bin/lld-link${CMAKE_EXECUTABLE_SUFFIX}"
    COMMAND ${CMAKE_COMMAND} -E copy
      "${LLVM_DIR}/bin/lld-link${CMAKE_EXECUTABLE_SUFFIX}"
      "${P4_DIR}/bin/lld-link${CMAKE_EXECUTABLE_SUFFIX}"
    COMMAND ${CMAKE_COMMAND} -E copy_directory
      "${LLVM_DIR}/lib/clang"
      "${P4_DIR}/lib/clang"
  )

  add_dependencies(deploy-${NAME}-main llvm-${NAME}-binaries_renaming)

  add_custom_target(deploy-${NAME} ALL
    COMMAND ${P4_EXECUTABLE} revert -a
      "${P4_DIR}/lib/clang/..."
    COMMAND ${P4_EXECUTABLE} reconcile -a
      "${P4_DIR}/lib/clang/..."
  )

  add_dependencies(deploy-${NAME} deploy-${NAME}-main)
endfunction()

set(LLVM20_TAG "llvmorg-20.1.8")
set(GIT_REPOSITORY "https://github.com/llvm/llvm-project.git")

set(DEPLOY_VERSION "20")

option(DO_LTO "Should we do LTO or not?" OFF)

build(NAME "llvm20" GIT_VERSION "${LLVM20_TAG}" PATCH_PROJECT_TO_USE "llvm${DEPLOY_VERSION}")

# Deploy only this version to the P4 location!
deploy(NAME "llvm${DEPLOY_VERSION}" VERSION "${DEPLOY_VERSION}")

