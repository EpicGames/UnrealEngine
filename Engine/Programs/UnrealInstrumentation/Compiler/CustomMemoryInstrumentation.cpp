// Copyright Epic Games, Inc. All Rights Reserved.

#include "llvm/Transforms/Instrumentation/CustomMemoryInstrumentation.h"
#include "llvm/ADT/SmallSet.h"
#include "llvm/ADT/SmallString.h"
#include "llvm/ADT/SmallVector.h"
#include "llvm/ADT/StringExtras.h"
#include "llvm/Analysis/CFG.h"
#include "llvm/Analysis/CallGraph.h"
#include "llvm/Analysis/CaptureTracking.h"
#include "llvm/Analysis/LoopInfo.h"
#include "llvm/Analysis/MemoryBuiltins.h"
#include "llvm/Analysis/ScalarEvolution.h"
#include "llvm/Analysis/TargetLibraryInfo.h"
#include "llvm/Analysis/TargetTransformInfo.h"
#include "llvm/Analysis/ValueTracking.h"
#include "llvm/Demangle/Demangle.h"
#include "llvm/IR/DataLayout.h"
#include "llvm/IR/DebugInfo.h"
#include "llvm/IR/DiagnosticInfo.h"
#include "llvm/IR/DiagnosticPrinter.h"
#include "llvm/IR/IRBuilder.h"
#include "llvm/IR/IntrinsicInst.h"
#include "llvm/IR/Intrinsics.h"
#include "llvm/IR/Module.h"
#include "llvm/IR/Verifier.h"
#include "llvm/IRReader/IRReader.h"
#include "llvm/InitializePasses.h"
#include "llvm/Linker/Linker.h"
#include "llvm/Support/CommandLine.h"
#include "llvm/Support/DynamicLibrary.h"
#include "llvm/Support/FileSystem.h"
#include "llvm/Support/ModRef.h"
#include "llvm/Support/Regex.h"
#include "llvm/Support/SourceMgr.h"
#if LLVM_VERSION_MAJOR >= 20
#include "llvm/Transforms/Utils/Instrumentation.h"
#else
#include "llvm/Transforms/Instrumentation.h"
#endif
#ifdef LLVM_HAS_AUTORTFM
#include "AutoRTFM/Constants.h"
#endif
#include "llvm/Transforms/Utils/BasicBlockUtils.h"
#include "llvm/Transforms/Utils/EscapeEnumerator.h"
#include "llvm/Transforms/Utils/Local.h"
#include "llvm/Transforms/Utils/LoopSimplify.h"
#include "llvm/Transforms/Utils/ModuleUtils.h"
#include "llvm/Transforms/Utils/ScalarEvolutionExpander.h"

namespace llvm {

// Atomic base (generic shared implementation).
Regex MSVCStdAtomicLoadRegex(".*std::_Atomic_storage.*::load.*");
Regex MSVCStdAtomicImplicitLoadRegex(
    ".*std::atomic<.*>::operator .*\\(void\\).*");
Regex MSVCStdAtomicStoreRegex(".*std::_Atomic_storage.*::store.*");
Regex MSVCStdAtomicExchangeRegex(".*std::_Atomic_storage.*::exchange.*");
Regex MSVCStdAtomicCompareExchangeRegex(
    ".*std::_Atomic_storage.*::compare_exchange_.*");

// Atomic integrals.
Regex MSVCStdAtomicFetchAddRegex(".*std::_Atomic_integral.*::fetch_add.*");
Regex MSVCStdAtomicFetchSubRegex(".*std::_Atomic_integral.*::fetch_sub.*");
Regex MSVCStdAtomicFetchAndRegex(".*std::_Atomic_integral.*::fetch_and.*");
Regex MSVCStdAtomicFetchOrRegex(".*std::_Atomic_integral.*::fetch_or.*");
Regex MSVCStdAtomicFetchXorRegex(".*std::_Atomic_integral.*::fetch_xor.*");

// Atomic pointers.
Regex
    MSVCStdAtomicPointerFetchAddRegex(".*std::_Atomic_pointer.*::fetch_add.*");
Regex
    MSVCStdAtomicPointerFetchSubRegex(".*std::_Atomic_pointer.*::fetch_sub.*");

SmallVector<std::pair<Regex *, AtomicCallSite>> MSVCAtomicCallSites = {
    {&MSVCStdAtomicLoadRegex, AtomicCallSite::LoadSite(0, 1)},
    {&MSVCStdAtomicImplicitLoadRegex, AtomicCallSite::LoadSite(0)},
    {&MSVCStdAtomicStoreRegex, AtomicCallSite::StoreSite(0, 1, 1, 2)},
    {&MSVCStdAtomicExchangeRegex, AtomicCallSite::ExchangeSite(0, 1, 1, 2)},
    {&MSVCStdAtomicCompareExchangeRegex,
     AtomicCallSite::CompareExchangeSite(0, 2, 1, 2, 3, 4)},
    {&MSVCStdAtomicFetchAddRegex,
     AtomicCallSite::RMWSite(AtomicRMWInst::BinOp::Add, 0, 1, 1, 2)},
    {&MSVCStdAtomicFetchSubRegex,
     AtomicCallSite::RMWSite(AtomicRMWInst::BinOp::Sub, 0, 1, 1, 2)},
    {&MSVCStdAtomicFetchAndRegex,
     AtomicCallSite::RMWSite(AtomicRMWInst::BinOp::And, 0, 1, 1, 2)},
    {&MSVCStdAtomicFetchOrRegex,
     AtomicCallSite::RMWSite(AtomicRMWInst::BinOp::Or, 0, 1, 1, 2)},
    {&MSVCStdAtomicFetchXorRegex,
     AtomicCallSite::RMWSite(AtomicRMWInst::BinOp::Xor, 0, 1, 1, 2)},

    // Atomic pointers FetchAdd and FetchSub require pointer arithmetic.
    {&MSVCStdAtomicPointerFetchAddRegex,
     AtomicCallSite::RMWSite(AtomicRMWInst::BinOp::Add, 0, 1, 1, 2, true)},
    {&MSVCStdAtomicPointerFetchSubRegex,
     AtomicCallSite::RMWSite(AtomicRMWInst::BinOp::Sub, 0, 1, 1, 2, true)},
};

uint32_t GetRealNumCallOperands(CallInst *Call) {
  return Call->getNumOperands() - 1;
}

// std::atomic_ref<T> instantiates the same _Atomic_storage / _Atomic_integral /
// _Atomic_pointer / _Atomic_floating templates that std::atomic<T> uses, but
// with a reference type (T&) as the first template argument. The _Storage
// member is then a reference field, which at the ABI level is a pointer, so
// the `this` argument of the resulting load/store/etc. call points at a
// pointer to T rather than at T itself. Detect this by walking the first
// template argument of the matched `std::_Atomic_*<...>` and checking whether
// it ends in `&` (after stripping trailing whitespace). The walk balances
// nested angle brackets, so atomic_ref<MyTemplate<int>> is handled correctly
// and atomic<MyTemplate<int>> is not misclassified.
bool IsMSVCAtomicRefCallTarget(StringRef DemangledName) {
  auto IsAsciiSpace = [](char C) {
    return C == ' ' || C == '\t' || C == '\n' || C == '\r';
  };
  auto FirstArgEndsInRef = [&](size_t Start, size_t End) {
    while (End > Start && IsAsciiSpace(DemangledName[End - 1])) {
      --End;
    }
    return End > Start && DemangledName[End - 1] == '&';
  };

  size_t NamePos = DemangledName.find("std::_Atomic_");
  if (NamePos == StringRef::npos) {
    return false;
  }
  size_t OpenAngle = DemangledName.find('<', NamePos);
  if (OpenAngle == StringRef::npos) {
    return false;
  }
  int Depth = 1;
  for (size_t i = OpenAngle + 1; i < DemangledName.size(); ++i) {
    char C = DemangledName[i];
    if (C == '<') {
      ++Depth;
    } else if (C == '>') {
      if (--Depth == 0) {
        return FirstArgEndsInRef(OpenAngle + 1, i);
      }
    } else if (C == ',' && Depth == 1) {
      return FirstArgEndsInRef(OpenAngle + 1, i);
    }
  }
  return false;
}

bool IsRMWOpHandled(RMWBinOp Op) {
  switch (Op) {
  case RMWBinOp::Add:
  case RMWBinOp::Sub:
  case RMWBinOp::And:
  case RMWBinOp::Or:
  case RMWBinOp::Xor:
    return true;
  default:
    return false;
  }
}

struct MemoryAccessFlags {
  uint8_t bAtomic : 1;
};

enum FAtomicMemoryOrder : int8_t {
  MEMORY_ORDER_RELAXED,
  MEMORY_ORDER_CONSUME,
  MEMORY_ORDER_ACQUIRE,
  MEMORY_ORDER_RELEASE,
  MEMORY_ORDER_ACQ_REL,
  MEMORY_ORDER_SEQ_CST
};

FAtomicMemoryOrder MemoryOrderFromLLVMOrdering(const AtomicOrdering &Ordering) {
  switch (Ordering) {
  case AtomicOrdering::Acquire:
    return MEMORY_ORDER_ACQUIRE;
  case AtomicOrdering::Release:
    return MEMORY_ORDER_RELEASE;
  case AtomicOrdering::AcquireRelease:
    return MEMORY_ORDER_ACQ_REL;
  case AtomicOrdering::Unordered:
  case AtomicOrdering::Monotonic:
    return MEMORY_ORDER_RELAXED;
  case AtomicOrdering::SequentiallyConsistent:
    return MEMORY_ORDER_SEQ_CST;
  case AtomicOrdering::NotAtomic:
    assert(false);
    break;
  }
  llvm_unreachable("Should have a memory order.");
}

FAtomicMemoryOrder MemoryOrderFromInst(Instruction *Inst) {
  AtomicOrdering Ordering = AtomicOrdering::NotAtomic;
  if (StoreInst *Store = dyn_cast<StoreInst>(Inst)) {
    Ordering = Store->getOrdering();
  } else if (LoadInst *Load = dyn_cast<LoadInst>(Inst)) {
    Ordering = Load->getOrdering();
  } else if (AtomicRMWInst *RMW = dyn_cast<AtomicRMWInst>(Inst)) {
    Ordering = RMW->getOrdering();
  }
  return MemoryOrderFromLLVMOrdering(Ordering);
}

static_assert(sizeof(MemoryAccessFlags) == sizeof(uint8_t));

MemoryAccessFlags GetMemoryAccessFlags(Instruction *Inst) {
  MemoryAccessFlags Flags;
  Flags.bAtomic = false;

  if (StoreInst *Store = dyn_cast<StoreInst>(Inst)) {
    if (Store->isAtomic()) {
      Flags.bAtomic = (getAtomicSyncScopeID(Inst) != SyncScope::SingleThread);
    }
  } else if (LoadInst *Store = dyn_cast<LoadInst>(Inst)) {
    if (Store->isAtomic()) {
      Flags.bAtomic = (getAtomicSyncScopeID(Inst) != SyncScope::SingleThread);
    }
  }

  return Flags;
}

std::string RMWOpName(RMWBinOp Op) {
  std::string Str = AtomicRMWInst::getOperationName(Op).str();
  Str[0] = std::toupper(Str[0]);
  return Str;
}

Value *CustomMemoryInstrumentationPass::CreateCast(IRBuilder<> &Builder,
                                                   Value *Val,
                                                   Type *DesiredType) {
  if (Val->getType() == DesiredType) {
    return Val;
  }

  if (Val->getType() == Builder.getInt1Ty() &&
      DesiredType == Builder.getInt8Ty()) {
    return Builder.CreateIntCast(Val, DesiredType, false);
  }
  if (Val->getType() == Builder.getInt8Ty() &&
      DesiredType == Builder.getInt1Ty()) {
    return Builder.CreateIntCast(Val, DesiredType, false);
  }

  uint32_t Size =
      CurrentModule->getDataLayout().getTypeStoreSize(Val->getType());
  uint32_t DesiredSize =
      CurrentModule->getDataLayout().getTypeStoreSize(DesiredType);
  if (Size == DesiredSize) {
    return Builder.CreateBitOrPointerCast(Val, DesiredType);
  }

  errs() << "Cast not supported\n";
  assert(false && "Cast not supported");
  return nullptr;
}

Type *GetSretType(CallInst *Call) {
  Type *Typ = nullptr;
  Function *Func = Call->getCalledFunction();
  if (Func->hasStructRetAttr()) {
    Typ = Func->getParamStructRetType(0);
    if (!Typ) {
      Typ = Func->getParamStructRetType(1);
    }
  }
  return Typ;
}

uint64_t
GetPointeeSizeFromMSVCAtomicPointerFetchAddCall(Function &MSVCFetchAdd) {
  // Find the 'mul' instruction that contains the pointee size.
  for (auto &BasicBlock : MSVCFetchAdd) {
    for (auto &Instruction : BasicBlock) {
      if (BinaryOperator *BinOp = dyn_cast<BinaryOperator>(&Instruction)) {
        if (BinOp->getOpcode() == Instruction::Mul) {
          Value *Op = BinOp->getOperand(1);
          if (ConstantInt *ConstOp = dyn_cast<ConstantInt>(Op)) {
            return ConstOp->getSExtValue();
          }
        }
      }
    }
  }

  return 0;
}

uint64_t
GetPointeeSizeFromMSVCAtomicPointerFetchSubCall(Function &MSVCFetchSub) {
  // MSVC's fetch_sub ends up calling fetch_add. Find the call to fetch_add.
  for (auto &BasicBlock : MSVCFetchSub) {
    for (auto &Instruction : BasicBlock) {
      if (CallInst *Call = dyn_cast<CallInst>(&Instruction)) {
        std::string FunctionName =
            demangle(Call->getCalledFunction()->getName().str());
        if (MSVCStdAtomicPointerFetchAddRegex.match(FunctionName)) {
          return GetPointeeSizeFromMSVCAtomicPointerFetchAddCall(
              *Call->getCalledFunction());
        }
      }
    }
  }

  return 0;
}

uint64_t
CustomMemoryInstrumentationPass::CacheOrGetPointeeSizeForMSVCAtomicPointerRMW(
    CallInst *MSVCCall, RMWBinOp RMWOp) {
  Function *MSVCFunction = MSVCCall->getCalledFunction();
  auto CachedPointeeSize = AtomicPointeeSizeCache.find(MSVCFunction);
  if (CachedPointeeSize != AtomicPointeeSizeCache.end()) {
    return CachedPointeeSize->second;
  }

  uint64_t PointeeSize = 0;
  if (RMWOp == RMWBinOp::Add) {
    PointeeSize =
        GetPointeeSizeFromMSVCAtomicPointerFetchAddCall(*MSVCFunction);
  } else {
    PointeeSize =
        GetPointeeSizeFromMSVCAtomicPointerFetchSubCall(*MSVCFunction);
  }
  AtomicPointeeSizeCache[MSVCFunction] = PointeeSize;
  return PointeeSize;
}

CustomMemoryInstrumentationPass::CustomMemoryInstrumentationPass(
    bool MSVCStandardLibPrepass)
    : MSVCStandardLibPrepass(MSVCStandardLibPrepass) {

  for (const auto &ExcludeRegex : Options.ExcludedFunctionNameRegexes.keys()) {
    CachedExcludedFunctionRegexes.push_back(Regex(ExcludeRegex));
  }
}

CustomMemoryInstrumentationPass::CustomMemoryInstrumentationPass(
    const CustomMemoryInstrumentationOptions &Options,
    bool MSVCStandardLibPrepass)
    : Options(Options), MSVCStandardLibPrepass(MSVCStandardLibPrepass) {}

PreservedAnalyses
CustomMemoryInstrumentationPass::run(Module &M, ModuleAnalysisManager &AM) {
  bool Instrumented = false;
  if (shouldInstrumentModule(M)) {
    CurrentModule = &M;
    cacheInstrumentationFunctions(M);

    if (MSVCStandardLibPrepass) {
      Instrumented |= instrumentMSVCStandardLib(M);
    } else {
      Instrumented = instrumentModule(M);
    }

    if (Instrumented && verifyModule(M, &errs())) {
      errs() << "Broken module\n"
          //<< M
          ;
      exit(1);
    }

    CurrentModule = nullptr;
  }

  return Instrumented ? PreservedAnalyses::none() : PreservedAnalyses::all();
}

bool CustomMemoryInstrumentationPass::shouldInstrumentModule(Module &M) {
  std::string Filename = M.getSourceFileName();
  SmallString<256> CurrentModuleFilename = StringRef(Filename);
  sys::fs::make_absolute(CurrentModuleFilename);

  bool Included = Options.IncludedModulesRegexes.empty();
  for (const auto &IncludeRegex : Options.IncludedModulesRegexes.keys()) {
    Regex Reg(IncludeRegex);
    if (Reg.match(CurrentModuleFilename)) {
      Included = true;
      break;
    }
  }

  if (!Included) {
    return false;
  }

  for (const auto &ExcludeRegex :
       Options.FurtherExcludedModulesRegexes.keys()) {
    Regex Reg(ExcludeRegex);
    if (Reg.match(CurrentModuleFilename)) {
      Included = false;
      break;
    }
  }

  return Included;
}

bool CustomMemoryInstrumentationPass::shouldInstrumentFunction(Function &F) {
  for (auto &InstrumentFunction : InstrumentFunctions) {
    if (F.getName() == InstrumentFunction->getCallee()->getName()) {
      return false;
    }
  }

  std::string DemangledName = demangle(F.getName().str());

  for (const auto &Reg : CachedExcludedFunctionRegexes) {
    if (Reg.match(DemangledName)) {
      return false;
    }
  }

  if (F.hasFnAttribute(Attribute::Naked)) {
    return false;
  }

  if (F.hasFnAttribute(Attribute::DisableSanitizerInstrumentation)) {
    return false;
  }
  
  // Apply the SanitizeThread attribute to any function we instrument
  // to prevent SimpleCFG to speculate some instructions and cause
  // race condition that wouldn't exist otherwise.
  // See llvm::mustSuppressSpeculation
  if (!F.hasFnAttribute(Attribute::SanitizeThread)) {
	  F.addFnAttr(Attribute::SanitizeThread);
  }

  return true;
}

void CustomMemoryInstrumentationPass::cacheInstrumentationFunctions(Module &M) {
  IRBuilder Builder(M.getContext());

  AttributeList Attrs;
  Attrs = Attrs.addFnAttribute(M.getContext(), Attribute::NoUnwind);
  Attrs = Attrs.addFnAttribute(M.getContext(), Attribute::NoInline);
  Attrs = Attrs.addFnAttribute(M.getContext(),
                               Attribute::DisableSanitizerInstrumentation);

  // Function entry/exit.
  FuncEntryInstrumentFunction =
      M.getOrInsertFunction("__Instrument_FuncEntry", Attrs,
                            Builder.getVoidTy(), Builder.getPtrTy());
  FuncExitInstrumentFunction = M.getOrInsertFunction(
      "__Instrument_FuncExit", Attrs, Builder.getVoidTy());

  InstrumentFunctions.push_back(&FuncEntryInstrumentFunction);
  InstrumentFunctions.push_back(&FuncExitInstrumentFunction);

  // Virtual Ptr Load and Store
  StoreVPtrInstrumentFunction =
      M.getOrInsertFunction("__Instrument_VPtr_Store", Attrs, Builder.getVoidTy(), 
		  Builder.getPtrTy(), Builder.getPtrTy());

  LoadVPtrInstrumentFunction =
      M.getOrInsertFunction("__Instrument_VPtr_Load", Attrs, Builder.getVoidTy(),
                            Builder.getPtrTy());

  // Non-atomic loads/stores.
  StoreInstrumentFunction =
      M.getOrInsertFunction("__Instrument_Store", Attrs, Builder.getVoidTy(),
                            Builder.getInt64Ty(), Builder.getInt32Ty());
  LoadInstrumentFunction =
      M.getOrInsertFunction("__Instrument_Load", Attrs, Builder.getVoidTy(),
                            Builder.getInt64Ty(), Builder.getInt32Ty());
  StoreRangeInstrumentFunction = M.getOrInsertFunction(
      "__Instrument_StoreRange", Attrs, Builder.getVoidTy(),
      Builder.getInt64Ty(), Builder.getInt32Ty());
  LoadRangeInstrumentFunction = M.getOrInsertFunction(
      "__Instrument_LoadRange", Attrs, Builder.getVoidTy(),
      Builder.getInt64Ty(), Builder.getInt32Ty());

  InstrumentFunctions.push_back(&StoreInstrumentFunction);
  InstrumentFunctions.push_back(&LoadInstrumentFunction);
  InstrumentFunctions.push_back(&StoreRangeInstrumentFunction);
  InstrumentFunctions.push_back(&LoadRangeInstrumentFunction);

  // Atomic operations.
  for (size_t i = 1; i <= MAX_ATOMIC_SIZE; i *= 2) {
    SmallString<64> FuncName("__Instrument_AtomicStore_int" + utostr(i * 8));
    AtomicStoreInstrumentFunctions[FunctionIndexFromSize(i)] =
        M.getOrInsertFunction(
            FuncName, Attrs, Builder.getVoidTy(),     // Return void.
#if LLVM_VERSION_MAJOR >= 20
            Builder.getPtrTy(),
#else
            Builder.getIntNTy(i * 8)->getPointerTo(),
#endif // Atomic pointer.
            Builder.getIntNTy(i * 8),                 // Value to store.
            Builder.getInt8Ty()                       // Memory order.
        );
  }

  for (size_t i = 1; i <= MAX_ATOMIC_SIZE; i *= 2) {
    SmallString<64> FuncName("__Instrument_AtomicLoad_int" + utostr(i * 8));
    AtomicLoadInstrumentFunctions[FunctionIndexFromSize(i)] =
        M.getOrInsertFunction(
            FuncName, Attrs, Builder.getIntNTy(i * 8), // Return loaded value.
#if LLVM_VERSION_MAJOR >= 20
            Builder.getPtrTy(),
#else
            Builder.getIntNTy(i * 8)->getPointerTo(),
#endif  // Atomic pointer.
            Builder.getInt8Ty()                        // Memory order.
        );
  }
  for (size_t i = 1; i <= MAX_ATOMIC_SIZE; i *= 2) {
    SmallString<64> FuncName("__Instrument_AtomicExchange_int" + utostr(i * 8));
    AtomicExchangeInstrumentFunctions[FunctionIndexFromSize(i)] =
        M.getOrInsertFunction(
            FuncName, Attrs, Builder.getIntNTy(i * 8), // Return previous value.
#if LLVM_VERSION_MAJOR >= 20
            Builder.getPtrTy(),
#else
            Builder.getIntNTy(i * 8)->getPointerTo(),
#endif  // Atomic pointer.
            Builder.getIntNTy(i * 8),                  // Value to store.
            Builder.getInt8Ty()                        // Memory order.
        );
  }
  for (size_t i = 1; i <= MAX_ATOMIC_SIZE; i *= 2) {
    SmallString<64> FuncName("__Instrument_AtomicCompareExchange_int" + utostr(i * 8));
    AtomicCompareExchangeInstrumentFunctions[FunctionIndexFromSize(i)] =
        M.getOrInsertFunction(
            FuncName, Attrs, Builder.getIntNTy(i * 8), // Return previous value.
#if LLVM_VERSION_MAJOR >= 20
            Builder.getPtrTy(),
#else
            Builder.getIntNTy(i * 8)->getPointerTo(),
#endif  // Atomic pointer.
#if LLVM_VERSION_MAJOR >= 20
            Builder.getPtrTy(),
#else
            Builder.getIntNTy(i * 8)->getPointerTo(),
#endif  // Expected pointer.
            Builder.getIntNTy(i * 8),                  // Value to store.
            Builder.getInt8Ty(),                       // Success memory order.
            Builder.getInt8Ty()                        // Failure memory order.
        );
  }

  for (size_t i = 1; i <= MAX_ATOMIC_SIZE; i *= 2) {
    for (unsigned b = 0; b < RMWBinOp::LAST_BINOP; ++b) {
      if (!IsRMWOpHandled((RMWBinOp)b)) {
        continue;
      }

      std::string OpName = RMWOpName((RMWBinOp)b);

      SmallString<64> FuncName("__Instrument_AtomicFetch");
      FuncName.append(OpName);
      FuncName.append("_int");
      FuncName.append(utostr(i * 8));
      AtomicRMWInstrumentFunctions[b][FunctionIndexFromSize(i)] =
          M.getOrInsertFunction(
              FuncName, Attrs,
              Builder.getIntNTy(i * 8),                 // Return previous value.
  #if LLVM_VERSION_MAJOR >= 20
            Builder.getPtrTy(),
#else
            Builder.getIntNTy(i * 8)->getPointerTo(),
#endif // Atomic pointer.
              Builder.getIntNTy(i * 8),                 // Value to add.
              Builder.getInt8Ty()                       // Memory order.
          );
    }

    AtomicRMWInstrumentFunctions[RMWBinOp::Xchg][FunctionIndexFromSize(i)] =
        AtomicExchangeInstrumentFunctions[FunctionIndexFromSize(i)];
  }

  for (size_t i = 0; i < NUM_ATOMIC_FUNCS; ++i) {
    InstrumentFunctions.push_back(&AtomicStoreInstrumentFunctions[i]);
    InstrumentFunctions.push_back(&AtomicLoadInstrumentFunctions[i]);
    InstrumentFunctions.push_back(&AtomicExchangeInstrumentFunctions[i]);
    InstrumentFunctions.push_back(&AtomicCompareExchangeInstrumentFunctions[i]);
    for (size_t b = 0; b < RMWBinOp::LAST_BINOP; ++b) {
      if (AtomicRMWInstrumentFunctions[b][i].getCallee() != nullptr) {
        InstrumentFunctions.push_back(&AtomicRMWInstrumentFunctions[b][i]);
      }
    }
  }

#ifdef LLVM_HAS_AUTORTFM
  // Mark all __Instrument_* functions as autortfm_mode_internal so that
  // AutoRTFM does not generate or execute closed variants of these
  // instrumentation functions.
  for (auto &F : M) {
    if (F.getName().starts_with("__Instrument_")) {
      F.addFnAttr(Attribute::get(
          M.getContext(), Attribute::AutoRTFM,
          static_cast<uint64_t>(autortfm_mode_internal)));
    }
  }
#endif
}

bool CustomMemoryInstrumentationPass::instrumentMSVCStandardLib(Module &M) {
  bool AnyInstrumented = false;
  SmallVector<std::pair<AtomicCallSite *, CallInst *>> Insts;
  for (auto &Function : M) {
    if (!shouldInstrumentFunction(Function)) {
      continue;
    }

    // Find calls.
    for (auto &BasicBlock : Function) {
      for (auto &Instruction : BasicBlock) {
        if (CallInst *Call = dyn_cast<CallInst>(&Instruction)) {
          if (!Call->getCalledFunction()) {
            continue;
          }

          std::string DemangledName =
              demangle(Call->getCalledFunction()->getName().str());

          // errs() << "Call: " << DemangledName << "\n";

          for (auto &[FunctionNameRegex, CallSite] : MSVCAtomicCallSites) {
            if (FunctionNameRegex->match(DemangledName)) {
              Insts.push_back({&CallSite, Call});
            }
          }
        }
      }
    }
  }

  // First, if there is any call to instrument that requires
  // pointer arithmetic, figure out the pointee sizes before
  // any instrumentation can interfere with that process.
  for (auto &[CallSite, Inst] : Insts) {
    if (CallSite->RequiresPointerArithmetic) {
      CacheOrGetPointeeSizeForMSVCAtomicPointerRMW(Inst, CallSite->RMWOp);
    }
  }

  // Instrument calls.
  for (auto &[CallSite, Inst] : Insts) {
    AnyInstrumented |= instrumentMSVCAtomicCallSite(Inst, *CallSite);
  }

  return AnyInstrumented;
}

bool CustomMemoryInstrumentationPass::instrumentModule(Module &M) {
  bool AnyInstrumented = false;
  for (auto &Function : M) {
    if (!shouldInstrumentFunction(Function)) {
      continue;
    }

    bool SkipNonAtomics = Function.hasFnAttribute("no_sanitize_thread");
    bool ContainsCalls = false;
    bool FunctionInstrumented = false;
    for (auto &BasicBlock : Function) {
      SmallVector<StoreInst *> Stores;
      SmallVector<LoadInst *> Loads;
      SmallVector<AtomicCmpXchgInst *> CompareExchanges;
      SmallVector<AtomicRMWInst *> RMWs;
      SmallVector<MemTransferInst *> MemTransfers;
      SmallVector<MemSetInst *> MemSets;
      for (auto &Instruction : BasicBlock) {
        if (StoreInst *Store = dyn_cast<StoreInst>(&Instruction)) {
          Stores.push_back(Store);
        } else if (LoadInst *Load = dyn_cast<LoadInst>(&Instruction)) {
          Loads.push_back(Load);
        } else if (AtomicCmpXchgInst *Cmp =
                       dyn_cast<AtomicCmpXchgInst>(&Instruction)) {
          CompareExchanges.push_back(Cmp);
        } else if (AtomicRMWInst *RMW = dyn_cast<AtomicRMWInst>(&Instruction)) {
          RMWs.push_back(RMW);
        } else if (MemTransferInst *MemCpy =
                       dyn_cast<MemTransferInst>(&Instruction)) {
          MemTransfers.push_back(MemCpy);
        } else if (MemSetInst *MemSet = dyn_cast<MemSetInst>(&Instruction)) {
          MemSets.push_back(MemSet);
        } else if (CallInst *Call = dyn_cast<CallInst>(&Instruction)) {
          ContainsCalls = true;
        }
      }

      for (auto *Inst : Stores) {
        FunctionInstrumented |= instrumentStore(Inst, SkipNonAtomics);
      }

      for (auto *Inst : Loads) {
        FunctionInstrumented |= instrumentLoad(Inst, SkipNonAtomics);
      }

      for (auto *Inst : CompareExchanges) {
        FunctionInstrumented |= instrumentCompareExchange(Inst);
      }

      for (auto *Inst : RMWs) {
        FunctionInstrumented |= instrumentRMW(Inst);
      }

      if (!SkipNonAtomics) {
        for (auto *Inst : MemTransfers) {
          FunctionInstrumented |= instrumentMemTransfer(Inst);
        }

        for (auto *Inst : MemSets) {
          FunctionInstrumented |= instrumentMemSet(Inst);
        }
      }

      // errs() << "Instrumented function " << Function.getName() << "\n";
      // if (verifyFunction(Function, &errs())) {
      //  errs() << "Broken function" << Function << "\n ";
      //  return true;
      //}
    }

    if (FunctionInstrumented || ContainsCalls) {
      FunctionInstrumented |= instrumentFunctionEntry(Function);
      FunctionInstrumented |= instrumentFunctionExit(Function);
    }

    AnyInstrumented |= FunctionInstrumented;
  }

  return AnyInstrumented;
}

bool CustomMemoryInstrumentationPass::instrumentFunctionEntry(Function &F) {
#if LLVM_VERSION_MAJOR >= 20
  InstrumentationIRBuilder Builder(&*F.getEntryBlock().getFirstNonPHIIt());
  Value *ReturnAddress = Builder.CreateCall(
      Intrinsic::getOrInsertDeclaration(F.getParent(), Intrinsic::returnaddress),
#else
  InstrumentationIRBuilder Builder(F.getEntryBlock().getFirstNonPHI());
  Value *ReturnAddress = Builder.CreateCall(
      Intrinsic::getDeclaration(F.getParent(), Intrinsic::returnaddress),
#endif
      Builder.getInt32(0));
  Builder.CreateCall(FuncEntryInstrumentFunction, ReturnAddress);
  return true;
}

bool CustomMemoryInstrumentationPass::instrumentFunctionExit(Function &F) {
  EscapeEnumerator EE(F, "instrumentation_cleanup",
                      /* HandleExceptions = */ false);
  while (IRBuilder<> *Builder = EE.Next()) {
    InstrumentationIRBuilder::ensureDebugInfo(*Builder, F);
    Builder->CreateCall(FuncExitInstrumentFunction, {});
  }
  return true;
}

FunctionCallee &
CustomMemoryInstrumentationPass::getInstrumentFunctionForMSVCAtomicCallSite(
    uint32_t Size, AtomicCallSite &CallSite) {
  switch (CallSite.Type) {
  case CALL_SITE_LOAD:
    return AtomicLoadInstrumentFunctions[FunctionIndexFromSize(Size)];
  case CALL_SITE_STORE:
    return AtomicStoreInstrumentFunctions[FunctionIndexFromSize(Size)];
  case CALL_SITE_EXCHANGE:
    return AtomicExchangeInstrumentFunctions[FunctionIndexFromSize(Size)];
  case CALL_SITE_COMPARE_EXCHANGE:
    return AtomicCompareExchangeInstrumentFunctions[FunctionIndexFromSize(
        Size)];
  case CALL_SITE_RMW:
    return AtomicRMWInstrumentFunctions[CallSite.RMWOp]
                                       [FunctionIndexFromSize(Size)];
  }
  llvm_unreachable("Should be handled");
}

bool CustomMemoryInstrumentationPass::instrumentMSVCAtomicCallSite(
    CallInst *Inst, AtomicCallSite CallSite) {
  Type *SretType = CallSite.AdjustCallSiteForSret(Inst);

  InstrumentationIRBuilder Builder(Inst);
  uint32_t Size = 0;
  if (SretType) {
    Size = CurrentModule->getDataLayout().getTypeStoreSize(SretType);
  } else if (CallSite.SizeTypeOperand == -1) {
    if (Inst->getType()->isVoidTy()) {
      errs() << "Void type: " << *Inst << "\n";
      errs() << demangle(Inst->getCalledFunction()->getName().str()) << "\n";
      errs() << demangle(Inst->getFunction()->getName().str()) << "\n";
    }
    Size = CurrentModule->getDataLayout().getTypeStoreSize(Inst->getType());
  } else {
    Size = CurrentModule->getDataLayout().getTypeStoreSize(
        Inst->getArgOperand(CallSite.SizeTypeOperand)->getType());
  }

  if (Size > MAX_ATOMIC_SIZE) {
    return false;
  }

  Value *RawPtr = Inst->getArgOperand(CallSite.PtrOperand);

  // For std::atomic_ref<T> (i.e. _Atomic_*<T &, ...>), the `this` pointer
  // points at a reference field rather than at T. The reference is stored as
  // a pointer at the ABI level, so load it once to recover the address that
  // the atomic operation actually targets — otherwise the runtime would record
  // accesses at the atomic_ref object's stack slot instead of at &T.
  if (IsMSVCAtomicRefCallTarget(
          demangle(Inst->getCalledFunction()->getName().str()))) {
    RawPtr = Builder.CreateAlignedLoad(
        Builder.getPtrTy(), RawPtr,
        Align(CurrentModule->getDataLayout().getPointerSize()));
  }

  Value *Ptr =
#if LLVM_VERSION_MAJOR >= 20
      Builder.CreatePointerCast(RawPtr, Builder.getPtrTy());
#else
      Builder.CreatePointerCast(RawPtr,
                                Builder.getIntNTy(Size * 8)->getPointerTo());
#endif

  Value *Val = nullptr;
  if (CallSite.StoreValueOperand) {
    Val = Inst->getArgOperand(*CallSite.StoreValueOperand);

    // If we're doing pointer arithmetic, we need to know the pointee's size
    // to multiply the value with.
    if (CallSite.RequiresPointerArithmetic) {
      uint64_t PointeeSize =
          CacheOrGetPointeeSizeForMSVCAtomicPointerRMW(Inst, CallSite.RMWOp);
      if (PointeeSize == 0) {
        errs() << "Failed to determine pointee size for atomic pointer RMW: "
               << *Inst << "\n";
        report_fatal_error(make_error<StringError>(
            "Failed to determine pointee size for atomic pointer RMW",
            inconvertibleErrorCode()));
      }
      Val = Builder.CreateMul(
          Val, ConstantInt::get(Builder.getInt64Ty(), PointeeSize));
    }
  }

  Value *Expected = nullptr;
  if (CallSite.ExpectedOperand) {
#if LLVM_VERSION_MAJOR >= 20
    Expected = Builder.CreatePointerCast(
        Inst->getArgOperand(*CallSite.ExpectedOperand),
        Builder.getPtrTy());
#else
    Expected = Builder.CreatePointerCast(
        Inst->getArgOperand(*CallSite.ExpectedOperand),
        Builder.getIntNTy(Size * 8)->getPointerTo());
#endif
  }

  Value *MemoryOrder = nullptr;
  if (CallSite.AtomicOrderOperand &&
      GetRealNumCallOperands(Inst) > *CallSite.AtomicOrderOperand) {
    Value *StdMemoryOrder = Inst->getArgOperand(*CallSite.AtomicOrderOperand);
    MemoryOrder =
        Builder.CreateIntCast(StdMemoryOrder, Builder.getInt8Ty(), true);
  } else {
    MemoryOrder = ConstantInt::get(Builder.getInt8Ty(),
                                   FAtomicMemoryOrder::MEMORY_ORDER_SEQ_CST);
  }

  Value *FailureMemoryOrder = MemoryOrder;
  if (CallSite.FailureAtomicOrderOperand &&
      GetRealNumCallOperands(Inst) > *CallSite.FailureAtomicOrderOperand) {
    Value *StdMemoryOrder =
        Inst->getArgOperand(*CallSite.FailureAtomicOrderOperand);
    FailureMemoryOrder =
        Builder.CreateIntCast(StdMemoryOrder, Builder.getInt8Ty(), true);
  }

  Value *Sret = nullptr;
  if (CallSite.SretOperand) {
    Sret = Inst->getArgOperand(*CallSite.SretOperand);
  }

  FunctionCallee &InstrumentFunction =
      getInstrumentFunctionForMSVCAtomicCallSite(Size, CallSite);

  if (!InstrumentFunction.getCallee()) {
    return false;
  }

  if (CallSite.Type == CALL_SITE_COMPARE_EXCHANGE) {
    assert(!Sret);
    return instrumentAtomicCompareExchangeMemoryInst(
        Builder, Inst, Ptr, Expected, Val, MemoryOrder, FailureMemoryOrder,
        InstrumentFunction, true /* return a single boolean value */);
  }

  return instrumentAtomicMemoryInst(Builder, Inst, Ptr, Val, FailureMemoryOrder,
                                    InstrumentFunction, Sret);
}

bool CustomMemoryInstrumentationPass::shouldInstrumentAddr(Value *Addr)
{
  // if the variable is on stack and is never captured, we don't need to instrument it.
  if (isa<AllocaInst>(getUnderlyingObject(Addr)) &&
	  !PointerMayBeCaptured(Addr, true, true)) 
  {
	  return false;
  }

  return true;
}

bool CustomMemoryInstrumentationPass::instrumentStore(StoreInst *Inst,
                                                      bool SkipNonAtomics) {
  InstrumentationIRBuilder Builder(Inst);

  Value *Addr = Inst->getPointerOperand();

  if (!shouldInstrumentAddr(Addr))
  {
    return false;
  }

  // Special case for virtual table pointer updates.
  if (MDNode *Metadata = Inst->getMetadata(LLVMContext::MD_tbaa)) {
    if (Metadata->isTBAAVtableAccess()) {
      Value *ValueOperand = Inst->getValueOperand();
      if (isa<VectorType>(ValueOperand->getType()))
        ValueOperand = Builder.CreateExtractElement(
            ValueOperand, ConstantInt::get(Builder.getInt32Ty(), 0));

      if (ValueOperand->getType()->isIntegerTy())
        ValueOperand = Builder.CreateIntToPtr(ValueOperand, Builder.getPtrTy());

	  Builder.CreateCall(StoreVPtrInstrumentFunction, {Addr, ValueOperand});
      return true;
    }
  }

  Value *Ptr =
      Builder.CreateCast(Instruction::CastOps::PtrToInt,
                         Inst->getPointerOperand(), Builder.getInt64Ty());
  uint32_t Size = CurrentModule->getDataLayout().getTypeStoreSize(
      Inst->getValueOperand()->getType());

  if (Inst->isAtomic()) {
    assert(Size <= MAX_ATOMIC_SIZE);

    Value *MemoryOrder =
        ConstantInt::get(Builder.getInt8Ty(), MemoryOrderFromInst(Inst));

    return instrumentAtomicMemoryInst(
        Builder, Inst, Inst->getPointerOperand(), Inst->getValueOperand(),
        MemoryOrder,
        AtomicStoreInstrumentFunctions[FunctionIndexFromSize(Size)], nullptr);
  } else if (SkipNonAtomics) {
    return false;
  }

  return instrumentMemoryInst(Builder, Inst->getDebugLoc(), Ptr, Size,
                              StoreInstrumentFunction);
}

bool CustomMemoryInstrumentationPass::instrumentLoad(LoadInst *Inst,
                                                     bool SkipNonAtomics) {
  InstrumentationIRBuilder Builder(Inst);
  
  Value* Addr = Inst->getPointerOperand();

  if (!shouldInstrumentAddr(Addr))
  {
     return false;
  }

  // Special case for virtual table pointer reads.
  if (MDNode *Metadata = Inst->getMetadata(LLVMContext::MD_tbaa)) {
    if (Metadata->isTBAAVtableAccess()) {
      Builder.CreateCall(LoadVPtrInstrumentFunction, Addr);
      return true;
    }
  }

  Value *Ptr =
      Builder.CreateCast(Instruction::CastOps::PtrToInt,
						 Addr, Builder.getInt64Ty());
  uint32_t Size =
      CurrentModule->getDataLayout().getTypeStoreSize(Inst->getType());

  if (Inst->isAtomic()) {
    assert(Size <= MAX_ATOMIC_SIZE);

    Value *MemoryOrder =
        ConstantInt::get(Builder.getInt8Ty(), MemoryOrderFromInst(Inst));

    return instrumentAtomicMemoryInst(
        Builder, Inst, Addr, nullptr /* value */,
        MemoryOrder, AtomicLoadInstrumentFunctions[FunctionIndexFromSize(Size)],
        nullptr);
  } else if (SkipNonAtomics) {
    return false;
  }

  return instrumentMemoryInst(Builder, Inst->getDebugLoc(), Ptr, Size,
                              LoadInstrumentFunction);
}

bool CustomMemoryInstrumentationPass::instrumentCompareExchange(
    AtomicCmpXchgInst *Inst) {
  InstrumentationIRBuilder Builder(Inst);

  uint32_t Size = CurrentModule->getDataLayout().getTypeStoreSize(
      Inst->getNewValOperand()->getType());
  assert(Size <= MAX_ATOMIC_SIZE);

  Value *SuccessMemoryOrder =
      ConstantInt::get(Builder.getInt8Ty(),
                       MemoryOrderFromLLVMOrdering(Inst->getSuccessOrdering()));
  Value *FailureMemoryOrder =
      ConstantInt::get(Builder.getInt8Ty(),
                       MemoryOrderFromLLVMOrdering(Inst->getFailureOrdering()));

  return instrumentAtomicCompareExchangeMemoryInst(
      Builder, Inst, Inst->getPointerOperand(), Inst->getCompareOperand(),
      Inst->getNewValOperand(), SuccessMemoryOrder, FailureMemoryOrder,
      AtomicCompareExchangeInstrumentFunctions[FunctionIndexFromSize(Size)],
      false /* return both old val and success bool*/);
}

bool CustomMemoryInstrumentationPass::instrumentRMW(AtomicRMWInst *Inst) {
  InstrumentationIRBuilder Builder(Inst);

  uint32_t Size = CurrentModule->getDataLayout().getTypeStoreSize(
      Inst->getValOperand()->getType());
  assert(Size <= MAX_ATOMIC_SIZE);

  Value *MemoryOrder =
      ConstantInt::get(Builder.getInt8Ty(), MemoryOrderFromInst(Inst));

  FunctionCallee &InstrumentFunction =
      AtomicRMWInstrumentFunctions[Inst->getOperation()]
                                  [FunctionIndexFromSize(Size)];
  if (InstrumentFunction.getCallee()) {
    return instrumentAtomicMemoryInst(
        Builder, Inst, Inst->getPointerOperand(), Inst->getValOperand(),
        MemoryOrder,
        AtomicRMWInstrumentFunctions[Inst->getOperation()]
                                    [FunctionIndexFromSize(Size)],
        nullptr);
  }

  return false;
}

bool CustomMemoryInstrumentationPass::instrumentMemoryInst(
    InstrumentationIRBuilder &Builder, const DebugLoc &DebugLoc, Value *Ptr,
    uint32_t Size, FunctionCallee &InstrumentFunction) {
  Builder.CreateCall(InstrumentFunction,
                     {Ptr, ConstantInt::get(Builder.getInt32Ty(), Size)});
  return true;
}

bool CustomMemoryInstrumentationPass::instrumentAtomicMemoryInst(
    InstrumentationIRBuilder &Builder, Instruction *Inst, Value *Ptr,
    Value *ValIfStore, Value *MemoryOrder, FunctionCallee &InstrumentFunction,
    Value *Sret) {
  const DebugLoc &DebugLoc = Inst->getDebugLoc();

  CallInst *CallInstruction = nullptr;
  Value *Ret = nullptr;
  if (ValIfStore) {
    Value *Val = CreateCast(
        Builder, ValIfStore,
        InstrumentFunction.getFunctionType()->getFunctionParamType(1));
    CallInstruction =
        Builder.CreateCall(InstrumentFunction, {Ptr, Val, MemoryOrder});
    CallInstruction->setDebugLoc(DebugLoc);
  } else {
    CallInstruction =
        Builder.CreateCall(InstrumentFunction, {Ptr, MemoryOrder});
    CallInstruction->setDebugLoc(DebugLoc);
  }
  if (Sret) {
    Ret = Builder.CreateStore(CallInstruction, Sret);
  } else {
    Ret = CreateCast(Builder, CallInstruction, Inst->getType());
  }

  Inst->replaceAllUsesWith(Ret);
  Ret->takeName(Inst);
  Inst->eraseFromParent();

  return true;
}

bool CustomMemoryInstrumentationPass::instrumentAtomicCompareExchangeMemoryInst(
    InstrumentationIRBuilder &Builder, Instruction *Inst, Value *Ptr,
    Value *Expected, Value *Val, Value *SuccessMemoryOrder,
    Value *FailureMemoryOrder, FunctionCallee &InstrumentFunction,
    bool ReturnOnlyBool) {
  const DebugLoc &DebugLoc = Inst->getDebugLoc();

  Value *ExpectedVal = nullptr;
  Value *ExpectedPtr = nullptr;
  if (Expected->getType()->isPointerTy()) {
    ExpectedPtr = Expected;
    ExpectedVal = Builder.CreateLoad(Val->getType(), Expected);
  } else {
    ExpectedVal = Expected;

    // Insert alloca at the beginning of the function.
    auto CurrentInsertPoint = Builder.GetInsertPoint();
    Builder.SetInsertPoint(
        &*Inst->getFunction()->getEntryBlock().getFirstInsertionPt());

    AllocaInst *ExpectedPtrAlloca = Builder.CreateAlloca(Val->getType());
    ExpectedPtrAlloca->setAlignment(llvm::Align(MAX_ATOMIC_SIZE));
    ExpectedPtr = ExpectedPtrAlloca;

    Builder.SetInsertPoint(Inst->getParent(), CurrentInsertPoint);
    Builder.CreateStore(ExpectedVal, ExpectedPtrAlloca);
  }

  Value *Ret = nullptr;
  Value *StoreVal =
      CreateCast(Builder, Val,
                 InstrumentFunction.getFunctionType()->getFunctionParamType(2));
  Value *PrevVal = Builder.CreateCall(
      InstrumentFunction,
      {Ptr, ExpectedPtr, StoreVal, SuccessMemoryOrder, FailureMemoryOrder});
  dyn_cast<CallInst>(PrevVal)->setDebugLoc(DebugLoc);

  // Compare bytes (reinterpret value as integer bytes).
  Value *Success = Builder.CreateICmpEQ(
      PrevVal, CreateCast(Builder, ExpectedVal, PrevVal->getType()));

  // Handle return value.
  if (ReturnOnlyBool) {
    Ret = Success;
    Ret = CreateCast(Builder, Success, Inst->getType());
  } else {
    AtomicCmpXchgInst *CmpXchg = dyn_cast<AtomicCmpXchgInst>(Inst);
    assert(CmpXchg);

    Type *PrevValType = CmpXchg->getNewValOperand()->getType();
    PrevVal = CreateCast(Builder, PrevVal, PrevValType);

    Ret = Builder.CreateInsertValue(PoisonValue::get(Inst->getType()), PrevVal,
                                    0);
    Ret = Builder.CreateInsertValue(Ret, Success, 1);
  }

  Inst->replaceAllUsesWith(Ret);
  Ret->takeName(Inst);
  Inst->eraseFromParent();

  return true;
}

bool CustomMemoryInstrumentationPass::instrumentMemTransfer(
    MemTransferInst *Inst) {
  InstrumentationIRBuilder Builder(Inst);

  instrumentMemoryInstRange(Builder, Inst->getDebugLoc(), Inst->getSource(),
                            Inst->getLength(), LoadRangeInstrumentFunction);
  instrumentMemoryInstRange(Builder, Inst->getDebugLoc(), Inst->getDest(),
                            Inst->getLength(), StoreRangeInstrumentFunction);

  return true;
}

bool CustomMemoryInstrumentationPass::instrumentMemSet(MemSetInst *Inst) {
  InstrumentationIRBuilder Builder(Inst);

  return instrumentMemoryInstRange(Builder, Inst->getDebugLoc(),
                                   Inst->getDest(), Inst->getLength(),
                                   StoreRangeInstrumentFunction);
}

bool CustomMemoryInstrumentationPass::instrumentMemoryInstRange(
    InstrumentationIRBuilder &Builder, const DebugLoc &DebugLoc, Value *Ptr,
    Value *Length, FunctionCallee &InstrumentFunction) {
  Value *Addr = Builder.CreatePtrToInt(Ptr, Builder.getInt64Ty());
  Value *Size = Builder.CreateIntCast(Length, Builder.getInt32Ty(), false);

  CallInst *Call = Builder.CreateCall(InstrumentFunction, {Addr, Size});
  Call->setDebugLoc(DebugLoc);

  return true;
}
} // namespace llvm