// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef LLVM_TRANSFORMS_CUSTOMMEMORYINSTRUMENTATION_H
#define LLVM_TRANSFORMS_CUSTOMMEMORYINSTRUMENTATION_H

#include "llvm/IR/PassManager.h"
#if LLVM_VERSION_MAJOR >= 20
#include "llvm/Transforms/Utils/Instrumentation.h"
#else
#include "llvm/Transforms/Instrumentation.h"
#endif
#include "llvm/ADT/StringSet.h"
#include "llvm/Support/Regex.h"
#include "llvm/ADT/SmallVector.h"
#include "llvm/IR/IntrinsicInst.h"

namespace llvm {

using RMWBinOp = AtomicRMWInst::BinOp;

Type *GetSretType(CallInst *Call);

enum AtomicCallSiteType {
  CALL_SITE_LOAD,
  CALL_SITE_STORE,
  CALL_SITE_EXCHANGE,
  CALL_SITE_COMPARE_EXCHANGE,
  CALL_SITE_RMW,
};

struct AtomicCallSite {
  AtomicCallSite(AtomicCallSiteType Type) : Type(Type) {}

  static AtomicCallSite LoadSite(uint32_t PtrOperand,
                                 std::optional<uint32_t> AtomicOrderOperand = std::nullopt) {
    AtomicCallSite CallSite(CALL_SITE_LOAD);
    CallSite.PtrOperand = PtrOperand;
    CallSite.AtomicOrderOperand = AtomicOrderOperand;
    return CallSite;
  }

  static AtomicCallSite StoreSite(uint32_t PtrOperand, uint32_t SizeTypeOperand,
                                  uint32_t StoreValueOperand,
                                  uint32_t AtomicOrderOperand) {
    AtomicCallSite CallSite(CALL_SITE_STORE);
    CallSite.PtrOperand = PtrOperand;
    CallSite.SizeTypeOperand = SizeTypeOperand;
    CallSite.StoreValueOperand = StoreValueOperand;
    CallSite.AtomicOrderOperand = AtomicOrderOperand;
    return CallSite;
  }

  static AtomicCallSite ExchangeSite(uint32_t PtrOperand,
                                     uint32_t SizeTypeOperand,
                                     uint32_t StoreValueOperand,
                                     uint32_t AtomicOrderOperand) {
    AtomicCallSite CallSite(CALL_SITE_EXCHANGE);
    CallSite.PtrOperand = PtrOperand;
    CallSite.SizeTypeOperand = SizeTypeOperand;
    CallSite.StoreValueOperand = StoreValueOperand;
    CallSite.AtomicOrderOperand = AtomicOrderOperand;
    return CallSite;
  }

  static AtomicCallSite
  CompareExchangeSite(uint32_t PtrOperand, uint32_t SizeTypeOperand,
                      uint32_t ExpectedOperand, uint32_t StoreValueOperand,
                      uint32_t SuccessAtomicOrderOperand,
                      std::optional<uint32_t> FailureAtomicOrderOperand) {
    AtomicCallSite CallSite(CALL_SITE_COMPARE_EXCHANGE);
    CallSite.PtrOperand = PtrOperand;
    CallSite.ExpectedOperand = ExpectedOperand;
    CallSite.SizeTypeOperand = SizeTypeOperand;
    CallSite.StoreValueOperand = StoreValueOperand;
    CallSite.AtomicOrderOperand = SuccessAtomicOrderOperand;
    CallSite.FailureAtomicOrderOperand = FailureAtomicOrderOperand
                                             ? FailureAtomicOrderOperand
                                             : SuccessAtomicOrderOperand;
    return CallSite;
  }

  static AtomicCallSite RMWSite(RMWBinOp Op, uint32_t PtrOperand,
                                uint32_t SizeTypeOperand, uint32_t ValueOperand,
                                uint32_t AtomicOrderOperand,
                                bool RequiresPointerArithmetic = false) {
    AtomicCallSite CallSite(CALL_SITE_RMW);
    CallSite.PtrOperand = PtrOperand;
    CallSite.RMWOp = Op;
    CallSite.SizeTypeOperand = SizeTypeOperand;
    CallSite.StoreValueOperand = ValueOperand;
    CallSite.AtomicOrderOperand = AtomicOrderOperand;
    CallSite.RequiresPointerArithmetic = RequiresPointerArithmetic;
    return CallSite;
  }

  // Returns struct type.
  Type *AdjustCallSiteForSret(CallInst *Inst) {
    Function *Func = Inst->getCalledFunction();
    if (!Func->hasStructRetAttr()) {
      return nullptr;
    }
    SretOperand = Func->getParamStructRetType(0) != nullptr ? 0 : 1;
    if (PtrOperand == 0 && SretOperand == 0) {
      PtrOperand = 1;
    }
    if (AtomicOrderOperand)
      AtomicOrderOperand = *AtomicOrderOperand + 1;
    if (StoreValueOperand)
      StoreValueOperand = *StoreValueOperand + 1;
    if (ExpectedOperand)
      ExpectedOperand = *ExpectedOperand + 1;
    if (FailureAtomicOrderOperand)
      FailureAtomicOrderOperand = *FailureAtomicOrderOperand + 1;

    return GetSretType(Inst);
  }

  AtomicCallSiteType Type;
  RMWBinOp RMWOp = RMWBinOp::BAD_BINOP;
  std::optional<uint32_t> SretOperand;
  uint32_t PtrOperand = 0;
  int32_t SizeTypeOperand = -1; // -1 for return value.

  std::optional<uint32_t> AtomicOrderOperand;

  // Store, exchange, compare-exchange and RMW specific.
  std::optional<uint32_t> StoreValueOperand;

  // Compare-exchange specific.
  std::optional<uint32_t> ExpectedOperand;
  std::optional<uint32_t> FailureAtomicOrderOperand;

  // Whether this operation requires pointer arithmetic (e.g. pointer fetch_add).
  bool RequiresPointerArithmetic = false;
};

struct CustomMemoryInstrumentationOptions {
  StringSet<> IncludedModulesRegexes;
  StringSet<> FurtherExcludedModulesRegexes;
  StringSet<> ExcludedFunctionNameRegexes;

  bool MSVCStandardLibPrepass;
};

class CustomMemoryInstrumentationPass
    : public PassInfoMixin<CustomMemoryInstrumentationPass> {
public:
 
  CustomMemoryInstrumentationPass(bool MSVCStandardLibPrepass);
  CustomMemoryInstrumentationPass(
      const CustomMemoryInstrumentationOptions &Options,
      bool MSVCStandardLibPrepass);
  PreservedAnalyses run(Module &M, ModuleAnalysisManager &AM);

private:
  bool instrumentModule(Module &M);
  bool shouldInstrumentAddr(Value* Addr);
  bool shouldInstrumentModule(Module &M);
  bool shouldInstrumentFunction(Function &F);

  bool instrumentMSVCStandardLib(Module &M);

  void cacheInstrumentationFunctions(Module &M);

  Value *CreateCast(IRBuilder<> &Builder, Value *Val, Type *DesiredType);

  bool instrumentFunctionEntry(Function &F);
  bool instrumentFunctionExit(Function &F);

  FunctionCallee &
  getInstrumentFunctionForMSVCAtomicCallSite(uint32_t Size,
                                             AtomicCallSite &CallSite);
  bool instrumentMSVCAtomicCallSite(CallInst *Inst, AtomicCallSite CallSite);
  bool instrumentStore(StoreInst *Inst, bool SkipNonAtomics);
  bool instrumentLoad(LoadInst *Inst, bool SkipNonAtomics);
  bool instrumentCompareExchange(AtomicCmpXchgInst *Inst);
  bool instrumentRMW(AtomicRMWInst *Inst);

  bool instrumentMemoryInst(InstrumentationIRBuilder &Builder,
                            const DebugLoc &DebugLoc, Value *Ptr, uint32_t Size,
                            FunctionCallee &InstrumentFunction);
  bool instrumentAtomicMemoryInst(InstrumentationIRBuilder &Builder,
                                  Instruction *Inst, Value *Ptr,
                                  Value *ValIfStore, Value *MemoryOrder,
                                  FunctionCallee &InstrumentFunction,
                                  Value *Sret);
  bool instrumentAtomicCompareExchangeMemoryInst(
      InstrumentationIRBuilder &Builder, Instruction *Inst, Value *Ptr,
      Value *Expected, Value *Val, Value *SuccessMemoryOrder,
      Value *FailureMemoryOrder, FunctionCallee &InstrumentFunction,
      bool ReturnOnlyBool);

  bool instrumentMemTransfer(MemTransferInst *Inst);
  bool instrumentMemSet(MemSetInst *Inst);

  bool instrumentMemoryInstRange(InstrumentationIRBuilder &Builder,
                                 const DebugLoc &DebugLoc, Value *Ptr,
                                 Value *Length,
                                 FunctionCallee &InstrumentFunction);

  CustomMemoryInstrumentationOptions Options;
  SmallVector<Regex, 10> CachedExcludedFunctionRegexes;

  bool MSVCStandardLibPrepass;
  Module *CurrentModule = nullptr;

  FunctionCallee FuncEntryInstrumentFunction;
  FunctionCallee FuncExitInstrumentFunction;

  FunctionCallee StoreInstrumentFunction;
  FunctionCallee LoadInstrumentFunction;
  FunctionCallee StoreVPtrInstrumentFunction;
  FunctionCallee LoadVPtrInstrumentFunction;
  FunctionCallee StoreRangeInstrumentFunction;
  FunctionCallee LoadRangeInstrumentFunction;
  

  static constexpr size_t MAX_ATOMIC_SIZE = 8;
  static constexpr size_t NUM_ATOMIC_FUNCS = (MAX_ATOMIC_SIZE == 16) ? 5 : 4;
  FunctionCallee AtomicStoreInstrumentFunctions[NUM_ATOMIC_FUNCS];
  FunctionCallee AtomicLoadInstrumentFunctions[NUM_ATOMIC_FUNCS];
  FunctionCallee AtomicExchangeInstrumentFunctions[NUM_ATOMIC_FUNCS];
  FunctionCallee AtomicCompareExchangeInstrumentFunctions[NUM_ATOMIC_FUNCS];
  FunctionCallee AtomicRMWInstrumentFunctions[RMWBinOp::LAST_BINOP]
                                             [NUM_ATOMIC_FUNCS];

  SmallVector<FunctionCallee *> InstrumentFunctions;

  size_t FunctionIndexFromSize(size_t Size) {
    switch (Size) {
    case 1:
      return 0;
    case 2:
      return 1;
    case 4:
      return 2;
    case 8:
      return 3;
    }
    return (size_t)-1;
  }

  // Atomic pointer's pointee size cache. Maps a MSVC std::atomic<T*>::fetch_add/fetch_sub function
  // to the size of the underlying pointer type, i.e. sizeof(T).
  std::unordered_map<Function *, uint64_t> AtomicPointeeSizeCache;

  uint64_t CacheOrGetPointeeSizeForMSVCAtomicPointerRMW(CallInst *MSVCCall,
                                                 RMWBinOp RMWOp);
};

} // end namespace llvm

#endif // LLVM_TRANSFORMS_CUSTOMMEMORYINSTRUMENTATION_H
