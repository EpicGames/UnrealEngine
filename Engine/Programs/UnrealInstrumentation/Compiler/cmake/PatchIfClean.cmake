# Copyright Epic Games, Inc. All Rights Reserved.
# This CMake script will patch only if the corresponding git repository at the
# working directory is clean.

if(NOT DEFINED UNREALINSTRUMENTATION_DIR)
  message(FATAL_ERROR "The CMake define 'UNREALINSTRUMENTATION_DIR' was not set!")
endif()

if(NOT DEFINED PATCH_FILE)
  message(FATAL_ERROR "The CMake define 'PATCH_FILE' was not set!")
endif()

if(NOT DEFINED PATCH_HASH_FILE)
  message(FATAL_ERROR "The CMake define 'PATCH_HASH_FILE' was not set!")
endif()

if(NOT DEFINED GIT_EXECUTABLE)
  message(FATAL_ERROR "The CMake define 'GIT_EXECUTABLE' was not set!")
endif()

if(NOT DEFINED Patch_EXECUTABLE)
  message(FATAL_ERROR "The CMake define 'Patch_EXECUTABLE' was not set!")
endif()

execute_process(
  COMMAND ${GIT_EXECUTABLE} status
  OUTPUT_VARIABLE OUTPUT
  COMMAND_ERROR_IS_FATAL ANY
)

string(FIND "${OUTPUT}" "working tree clean" IS_CLEAN)

if(IS_CLEAN EQUAL -1)
  if(EXISTS "${PATCH_HASH_FILE}")
    file(READ "${PATCH_HASH_FILE}" CACHED_PATCH_HASH)
    file(SHA512 "${PATCH_FILE}" LATEST_PATCH_HASH)

    message(STATUS "Comparing '${CACHED_PATCH_HASH}' vs '${LATEST_PATCH_HASH}'")

    if(NOT "${CACHED_PATCH_HASH}" STREQUAL "${LATEST_PATCH_HASH}")
      message(STATUS "Hashes did not match, cleaning out git sources!")

      # Clean out the git sources.
      execute_process(
        COMMAND ${GIT_EXECUTABLE} checkout .
        COMMAND ${GIT_EXECUTABLE} clean -fd
        COMMAND_ERROR_IS_FATAL ANY
      )

      # We need to force repatching.
      set(IS_CLEAN 0)
    endif()
  else()
    message(STATUS "Hashes did not match, cleaning out git sources!")

    # Clean out the git sources.
    execute_process(
      COMMAND ${GIT_EXECUTABLE} checkout .
      COMMAND ${GIT_EXECUTABLE} clean -fd
      COMMAND_ERROR_IS_FATAL ANY
    )
    
    # We need to force repatching.
    set(IS_CLEAN 0)
  endif()
endif()

if(NOT IS_CLEAN EQUAL -1)
  execute_process(
    COMMAND ${CMAKE_COMMAND} -E copy_if_different "${UNREALINSTRUMENTATION_DIR}/Compiler/CustomMemoryInstrumentation.cpp" "llvm/lib/Transforms/Instrumentation"
    COMMAND ${CMAKE_COMMAND} -E copy_if_different "${UNREALINSTRUMENTATION_DIR}/Compiler/CustomMemoryInstrumentation.h" "llvm/include/llvm/Transforms/Instrumentation"
    COMMAND_ERROR_IS_FATAL ANY
  )

  execute_process(
    COMMAND ${Patch_EXECUTABLE} -p1 -i ${PATCH_FILE}
    COMMAND_ERROR_IS_FATAL ANY
  )
else()
  message(STATUS "Git sources are already patched so not re-applying!")
endif()
