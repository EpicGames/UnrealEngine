diff --git a/clang/lib/CodeGen/BackendUtil.cpp b/clang/lib/CodeGen/BackendUtil.cpp
index 4f22d35f9d3a..4b5d2f740b23 100644
--- a/clang/lib/CodeGen/BackendUtil.cpp
+++ b/clang/lib/CodeGen/BackendUtil.cpp
@@ -67,6 +67,7 @@
 #include "llvm/Transforms/Instrumentation.h"
 #include "llvm/Transforms/Instrumentation/AddressSanitizer.h"
 #include "llvm/Transforms/Instrumentation/AddressSanitizerOptions.h"
+#include "llvm/Transforms/Instrumentation/CustomMemoryInstrumentation.h"
 #include "llvm/Transforms/Instrumentation/BoundsChecking.h"
 #include "llvm/Transforms/Instrumentation/DataFlowSanitizer.h"
 #include "llvm/Transforms/Instrumentation/GCOVProfiler.h"
@@ -998,6 +999,17 @@ void EmitAssemblyHelper::RunOptimizationPipeline(
           [Options](ModulePassManager &MPM, OptimizationLevel Level) {
             MPM.addPass(InstrProfilingLoweringPass(*Options, false));
           });
+    
+	// MSVC standard lib pre-pass (to instrument atomics).
+    PB.registerPipelineStartEPCallback(
+        [](ModulePassManager &MPM, OptimizationLevel Level) {
+          MPM.addPass(CustomMemoryInstrumentationPass(true));
+        });
+
+	PB.registerOptimizerLastEPCallback(
+        [](ModulePassManager &MPM, OptimizationLevel Level) {
+          MPM.addPass(CustomMemoryInstrumentationPass(false));
+        });
 
     // TODO: Consider passing the MemoryProfileOutput to the pass builder via
     // the PGOOptions, and set this up there.
diff --git a/clang/lib/CodeGen/CodeGenModule.cpp b/clang/lib/CodeGen/CodeGenModule.cpp
index eb13cd40eb8a..1b6388f681d0 100644
--- a/clang/lib/CodeGen/CodeGenModule.cpp
+++ b/clang/lib/CodeGen/CodeGenModule.cpp
@@ -394,10 +394,8 @@ CodeGenModule::CodeGenModule(ASTContext &C,
     createHLSLRuntime();
 
   // Enable TBAA unless it's suppressed. ThreadSanitizer needs TBAA even at O0.
-  if (LangOpts.Sanitize.has(SanitizerKind::Thread) ||
-      (!CodeGenOpts.RelaxedAliasing && CodeGenOpts.OptimizationLevel > 0))
-    TBAA.reset(new CodeGenTBAA(Context, TheModule, CodeGenOpts, getLangOpts(),
-                               getCXXABI().getMangleContext()));
+	TBAA.reset(new CodeGenTBAA(Context, TheModule, CodeGenOpts, getLangOpts(),
+								getCXXABI().getMangleContext()));
 
   // If debug info or coverage generation is enabled, create the CGDebugInfo
   // object.
diff --git a/clang/lib/CodeGen/MicrosoftCXXABI.cpp b/clang/lib/CodeGen/MicrosoftCXXABI.cpp
index 4d0f4c63f843..350996eeca1e 100644
--- a/clang/lib/CodeGen/MicrosoftCXXABI.cpp
+++ b/clang/lib/CodeGen/MicrosoftCXXABI.cpp
@@ -2730,7 +2730,7 @@ void MicrosoftCXXABI::EmitGuardedInit(CodeGenFunction &CGF, const VarDecl &D,
 
     // This BasicBLock determines whether or not we have any work to do.
     llvm::LoadInst *FirstGuardLoad = Builder.CreateLoad(GuardAddr);
-    FirstGuardLoad->setOrdering(llvm::AtomicOrdering::Unordered);
+    FirstGuardLoad->setOrdering(llvm::AtomicOrdering::Acquire);
     llvm::LoadInst *InitThreadEpoch =
         Builder.CreateLoad(getInitThreadEpochPtr(CGM));
     llvm::Value *IsUninitialized =
@@ -2746,7 +2746,7 @@ void MicrosoftCXXABI::EmitGuardedInit(CodeGenFunction &CGF, const VarDecl &D,
     CGF.EmitNounwindRuntimeCall(getInitThreadHeaderFn(CGM),
                                 GuardAddr.getPointer());
     llvm::LoadInst *SecondGuardLoad = Builder.CreateLoad(GuardAddr);
-    SecondGuardLoad->setOrdering(llvm::AtomicOrdering::Unordered);
+    SecondGuardLoad->setOrdering(llvm::AtomicOrdering::Acquire);
     llvm::Value *ShouldDoInit =
         Builder.CreateICmpEQ(SecondGuardLoad, getAllOnesInt());
     llvm::BasicBlock *InitBlock = CGF.createBasicBlock("init");
@@ -2757,8 +2757,18 @@ void MicrosoftCXXABI::EmitGuardedInit(CodeGenFunction &CGF, const VarDecl &D,
     CGF.EHStack.pushCleanup<CallInitThreadAbort>(EHCleanup, GuardAddr);
     CGF.EmitCXXGlobalVarDeclInit(D, GV, PerformInit);
     CGF.PopCleanupBlock();
+
+	llvm::Value *Zero = llvm::Constant::getNullValue(GuardTy);
+    // Provides a proper barrier after the initialization on the slow path.
+    // We don't care about failure we just want a release barrier on the
+    // GuardAddr that the other threads can acquire.
+    Builder.CreateAtomicCmpXchg(GuardAddr, Zero, Zero,
+                                llvm::AtomicOrdering::SequentiallyConsistent,
+                                llvm::AtomicOrdering::SequentiallyConsistent);
+
     CGF.EmitNounwindRuntimeCall(getInitThreadFooterFn(CGM),
                                 GuardAddr.getPointer());
+
     Builder.CreateBr(EndBlock);
 
     CGF.EmitBlock(EndBlock);
diff --git a/llvm/lib/Transforms/Instrumentation/CMakeLists.txt b/llvm/lib/Transforms/Instrumentation/CMakeLists.txt
index 424f1d433606..089644655402 100644
--- a/llvm/lib/Transforms/Instrumentation/CMakeLists.txt
+++ b/llvm/lib/Transforms/Instrumentation/CMakeLists.txt
@@ -6,6 +6,7 @@ add_llvm_component_library(LLVMInstrumentation
   DataFlowSanitizer.cpp
   GCOVProfiling.cpp
   BlockCoverageInference.cpp
+  CustomMemoryInstrumentation.cpp
   MemProfiler.cpp
   MemorySanitizer.cpp
   IndirectCallPromotion.cpp
