diff -ruN orig/clang/lib/CodeGen/BackendUtil.cpp modified/clang/lib/CodeGen/BackendUtil.cpp
--- a/clang/lib/CodeGen/BackendUtil.cpp	2026-03-25 12:05:30.904749000 -0400
+++ b/clang/lib/CodeGen/BackendUtil.cpp	2026-03-25 12:06:48.542480200 -0400
@@ -63,6 +63,7 @@
 #include "llvm/Transforms/InstCombine/InstCombine.h"
 #include "llvm/Transforms/Instrumentation/AddressSanitizer.h"
 #include "llvm/Transforms/Instrumentation/AddressSanitizerOptions.h"
+#include "llvm/Transforms/Instrumentation/CustomMemoryInstrumentation.h"
 #include "llvm/Transforms/Instrumentation/BoundsChecking.h"
 #include "llvm/Transforms/Instrumentation/DataFlowSanitizer.h"
 #include "llvm/Transforms/Instrumentation/GCOVProfiler.h"
@@ -1071,6 +1072,17 @@
             MPM.addPass(InstrProfilingLoweringPass(*Options, false));
           });
 
+	// MSVC standard lib pre-pass (to instrument atomics).
+    PB.registerPipelineStartEPCallback(
+        [](ModulePassManager &MPM, OptimizationLevel Level) {
+          MPM.addPass(CustomMemoryInstrumentationPass(true));
+        });
+
+	PB.registerOptimizerLastEPCallback(
+        [](ModulePassManager &MPM, OptimizationLevel Level, ThinOrFullLTOPhase) {
+          MPM.addPass(CustomMemoryInstrumentationPass(false));
+        });
+
     // TODO: Consider passing the MemoryProfileOutput to the pass builder via
     // the PGOOptions, and set this up there.
     if (!CodeGenOpts.MemoryProfileOutput.empty()) {
diff -ruN orig/clang/lib/CodeGen/CodeGenModule.cpp modified/clang/lib/CodeGen/CodeGenModule.cpp
--- a/clang/lib/CodeGen/CodeGenModule.cpp	2026-03-25 12:05:30.923768700 -0400
+++ b/clang/lib/CodeGen/CodeGenModule.cpp	2026-03-25 12:07:34.317298200 -0400
@@ -401,10 +401,8 @@
     createHLSLRuntime();
 
   // Enable TBAA unless it's suppressed. TSan and TySan need TBAA even at O0.
-  if (LangOpts.Sanitize.hasOneOf(SanitizerKind::Thread | SanitizerKind::Type) ||
-      (!CodeGenOpts.RelaxedAliasing && CodeGenOpts.OptimizationLevel > 0))
-    TBAA.reset(new CodeGenTBAA(Context, getTypes(), TheModule, CodeGenOpts,
-                               getLangOpts()));
+	TBAA.reset(new CodeGenTBAA(Context, getTypes(), TheModule, CodeGenOpts,
+							getLangOpts()));
 
   // If debug info or coverage generation is enabled, create the CGDebugInfo
   // object.
diff -ruN orig/clang/lib/CodeGen/MicrosoftCXXABI.cpp modified/clang/lib/CodeGen/MicrosoftCXXABI.cpp
--- a/clang/lib/CodeGen/MicrosoftCXXABI.cpp	2026-03-25 12:05:31.059626600 -0400
+++ b/clang/lib/CodeGen/MicrosoftCXXABI.cpp	2026-03-25 12:43:05.643986600 -0400
@@ -2737,7 +2737,7 @@
 
     // This BasicBLock determines whether or not we have any work to do.
     llvm::LoadInst *FirstGuardLoad = Builder.CreateLoad(GuardAddr);
-    FirstGuardLoad->setOrdering(llvm::AtomicOrdering::Unordered);
+    FirstGuardLoad->setOrdering(llvm::AtomicOrdering::Acquire);
     llvm::LoadInst *InitThreadEpoch =
         Builder.CreateLoad(getInitThreadEpochPtr(CGM));
     llvm::Value *IsUninitialized =
@@ -2753,7 +2753,7 @@
     CGF.EmitNounwindRuntimeCall(getInitThreadHeaderFn(CGM),
                                 GuardAddr.getPointer());
     llvm::LoadInst *SecondGuardLoad = Builder.CreateLoad(GuardAddr);
-    SecondGuardLoad->setOrdering(llvm::AtomicOrdering::Unordered);
+    SecondGuardLoad->setOrdering(llvm::AtomicOrdering::Acquire);
     llvm::Value *ShouldDoInit =
         Builder.CreateICmpEQ(SecondGuardLoad, getAllOnesInt());
     llvm::BasicBlock *InitBlock = CGF.createBasicBlock("init");
@@ -2764,6 +2764,15 @@
     CGF.EHStack.pushCleanup<CallInitThreadAbort>(EHCleanup, GuardAddr);
     CGF.EmitCXXGlobalVarDeclInit(D, GV, PerformInit);
     CGF.PopCleanupBlock();
+
+	llvm::Value *Zero = llvm::Constant::getNullValue(GuardTy);
+    // Provides a proper barrier after the initialization on the slow path.
+    // We don't care about failure we just want a release barrier on the
+    // GuardAddr that the other threads can acquire.
+    Builder.CreateAtomicCmpXchg(GuardAddr, Zero, Zero,
+                                llvm::AtomicOrdering::SequentiallyConsistent,
+                                llvm::AtomicOrdering::SequentiallyConsistent);
+
     CGF.EmitNounwindRuntimeCall(getInitThreadFooterFn(CGM),
                                 GuardAddr.getPointer());
     Builder.CreateBr(EndBlock);
diff -ruN orig/llvm/lib/Transforms/Instrumentation/CMakeLists.txt modified/llvm/lib/Transforms/Instrumentation/CMakeLists.txt
--- a/llvm/lib/Transforms/Instrumentation/CMakeLists.txt	2026-03-25 12:05:30.965799800 -0400
+++ b/llvm/lib/Transforms/Instrumentation/CMakeLists.txt	2026-03-25 12:43:19.070128800 -0400
@@ -3,6 +3,7 @@
   BoundsChecking.cpp
   CGProfile.cpp
   ControlHeightReduction.cpp
+  CustomMemoryInstrumentation.cpp
   DataFlowSanitizer.cpp
   GCOVProfiling.cpp
   BlockCoverageInference.cpp
