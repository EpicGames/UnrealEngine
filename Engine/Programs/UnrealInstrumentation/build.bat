@rem Copyright Epic Games, Inc. All Rights Reserved.

@echo off
setlocal enabledelayedexpansion

where /q cl.exe || ECHO Cound not find 'cl.exe' on the path. Have you ran 'vcvarsall.bat'?. && EXIT /B

set UNREALINSTRUMENTATION_DIR=%~dp0
set ENGINE_DIR=%UNREALINSTRUMENTATION_DIR%\..\..

set UNREALINSTRUMENTATION_SCRATCH=%ENGINE_DIR%\Intermediate\UnrealInstrumentation

rem Do the LLVM builds that make our custom clang-cl's.
mkdir %UNREALINSTRUMENTATION_SCRATCH%
cd %UNREALINSTRUMENTATION_SCRATCH%
cmake -G Ninja %UNREALINSTRUMENTATION_DIR%\Compiler
if %errorlevel% neq 0 goto error
ninja -C %UNREALINSTRUMENTATION_SCRATCH%
if %errorlevel% neq 0 goto error

rem If we update to a later LLVM, we should update the version used here!
set UNREALINSTRUMENTATION_CUSTOM_CLANG_DIR=%UNREALINSTRUMENTATION_DIR%\..\..\Binaries\Win64\UnrealInstrumentation
set "UNREALINSTRUMENTATION_CUSTOM_CLANG_DIR=%UNREALINSTRUMENTATION_CUSTOM_CLANG_DIR:\=/%"

exit /b 0

:error
echo Error building
exit /b 1
